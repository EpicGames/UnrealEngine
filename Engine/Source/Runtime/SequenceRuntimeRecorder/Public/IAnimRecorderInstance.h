// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "IAnimationRecorder.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

//////////////////////////////////////////////////////////////////////////
// IAnimRecorderInstance

namespace UE::AnimationRecording
{

/**
 * Base interface for animation recorder instances.
 * Manages the lifecycle of a single recording session for a skeletal mesh component.
 */
struct IAnimRecorderInstance
{
public:
	virtual ~IAnimRecorderInstance() = default;

	/** Initializes the instance to record into an existing animation sequence. */
	virtual void Init(USkeletalMeshComponent* InComponent, UAnimSequence* InSequence, FAnimationSerializer* InAnimationSerializer, const FAnimationRecordingSettings& InSettings) = 0;

	/** Starts the recording session. Returns true on success. */
	virtual bool BeginRecording() = 0;

	/** Records a single frame of pose data. Call each tick while recording. */
	virtual void Update(float DeltaTime) = 0;

	/** Stops recording and finalizes the animation sequence. */
	virtual void FinishRecording(bool bShowMessage = true) = 0;

	/** Process any time data captured and apply it to the bones on the given SkeletalMeshComponent. */
	virtual void ProcessRecordedTimes(
		UAnimSequence* AnimSequence, USkeletalMeshComponent* SkeletalMeshComponent,
		const FTimecodeBoneMethod& TimecodeBoneMethod, const FProcessRecordedTimeParams& TimecodeInfo) = 0;

	virtual TSharedPtr<UE::AnimationRecording::IAnimationRecorder> GetRecorder() const = 0;
};

} // namespace UE::AnimationRecording
