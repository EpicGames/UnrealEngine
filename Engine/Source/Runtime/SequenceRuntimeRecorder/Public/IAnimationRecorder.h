// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Logging/LogCategory.h"
#include "Math/Transform.h"
#include "Misc/FrameRate.h"
#include "Misc/Optional.h"
#include "Misc/QualifiedFrameTime.h"
#include "Templates/Function.h"
#include "TimecodeBoneMethod.h"
#include "Serializers/MovieSceneAnimationSerialization.h"

class UAnimBoneCompressionSettings;
class UAnimSequence;
class USkeletalMeshComponent;
struct FAnimationRecordingSettings;

struct FProcessRecordedTimeParams
{
	FString HoursName;
	FString MinutesName;
	FString SecondsName;
	FString FramesName;
	FString SubFramesName;

	// Optionally support writing the Timecode Rate into time data.
	TOptional<FString> RateName;

	FString SlateName;
	FString Slate;
};

//////////////////////////////////////////////////////////////////////////
// IAnimationRecorder

namespace UE::AnimationRecording
{

/**
 * Base interface for animation recorders.
 * Inheriting from this interface allows different animation recording implementations as required.
 */
struct IAnimationRecorder
{
	virtual ~IAnimationRecorder() = default;

	/** Begins recording pose data from the component into the given animation sequence. */
	virtual void StartRecord(USkeletalMeshComponent* Component, UAnimSequence* InAnimationObject) = 0;

	/** Stops recording and finalizes the animation sequence. Returns the recorded sequence. */
	virtual UAnimSequence* StopRecord(bool bShowMessage) = 0;

	/** Records a single frame of pose data from the component. Call each tick while recording. */
	virtual void UpdateRecord(USkeletalMeshComponent* Component, float DeltaTime) = 0;

	/** Returns true if a recording is currently in progress. */
	virtual bool InRecording() const = 0;

	/** Sets the function that should be used to get the current timecode for the current frame. */
	virtual void SetCurrentFrameTimeGetter(TFunction<TOptional<FQualifiedFrameTime>()> InGetCurrentTimeFunction) = 0;

	/** Returns the root bone transform captured at the start of recording. */
	virtual const FTransform& GetInitialRootTransform() const = 0;

	/** Process any time data captured and apply it to the bones on the given SkeletalMeshComponent. */
	virtual void ProcessRecordedTimes(
		UAnimSequence* AnimSequence, USkeletalMeshComponent* SkeletalMeshComponent,
		const FTimecodeBoneMethod& TimecodeBoneMethod, const FProcessRecordedTimeParams& TimecodeInfo) = 0;
};

} // namespace UE::AnimationRecording
