// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/NameTypes.h"

#include "TimecodeBoneMethod.generated.h"

UENUM(BlueprintType)
enum class ETimecodeBoneMode : uint8
{
	All,
	Root,
	UserDefined,
	MAX UMETA(Hidden)
};

USTRUCT(BlueprintType)
struct FTimecodeBoneMethod
{
	GENERATED_USTRUCT_BODY()

	/** Default constructor, initializing with default values */
	FTimecodeBoneMethod() : BoneMode(ETimecodeBoneMode::Root) { }

	/** The timecode bone mode */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Timecode")
	ETimecodeBoneMode BoneMode;

	/** Name of the bone to assign timecode values to */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Timecode")
	FName BoneName;
};
