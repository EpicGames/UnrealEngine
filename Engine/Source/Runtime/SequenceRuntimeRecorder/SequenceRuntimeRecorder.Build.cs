// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class SequenceRuntimeRecorder : ModuleRules
	{
		public SequenceRuntimeRecorder(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"TimeManagement",
                    "SerializedRecorderInterface",

                }
            );

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"SlateCore",
					"Slate",
					"InputCore",
					"Engine",
                }
            );
		}
	}
}
